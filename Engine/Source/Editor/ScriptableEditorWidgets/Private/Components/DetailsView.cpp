// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DetailsView.h"

#include "Components/PropertyViewHelper.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "HAL/Platform.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DetailsView)

class UObject;

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UDetailsView


void UDetailsView::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	DetailViewWidget.Reset();
}


void UDetailsView::BuildContentWidget()
{
	DetailViewWidget.Reset();

	if (!GetDisplayWidget().IsValid())
	{
		return;
	}

	bool bCreateMissingWidget = true;
	FText MissingWidgetText = FPropertyViewHelper::EditorOnlyText;

	if (GIsEditor)
	{
		UObject* ViewedObject = GetObject();
		if (ViewedObject == nullptr)
		{
			bool bIsObjectNull = Object.IsNull();
			if (bIsObjectNull)
			{
				MissingWidgetText = FPropertyViewHelper::UndefinedObjectText;
			}
			else
			{
				MissingWidgetText = FPropertyViewHelper::UnloadedObjectText;
			}
		}
		else
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bShowPropertyMatrixButton = false;
			DetailsViewArgs.NotifyHook = this;

			DetailsViewArgs.ViewIdentifier = ViewIdentifier;
			DetailsViewArgs.bAllowSearch = bAllowFiltering;
			DetailsViewArgs.bAllowFavoriteSystem = bAllowFavoriteSystem;
			DetailsViewArgs.bShowOptions = bAllowFiltering;
			DetailsViewArgs.bShowModifiedPropertiesOption = bShowModifiedPropertiesOption;
			DetailsViewArgs.bShowKeyablePropertiesOption = bShowKeyablePropertiesOption;
			DetailsViewArgs.bShowAnimatedPropertiesOption = bShowAnimatedPropertiesOption;
			DetailsViewArgs.bShowScrollBar = bShowScrollBar;
			DetailsViewArgs.bForceHiddenPropertyVisibility = bForceHiddenPropertyVisibility;
			DetailsViewArgs.ColumnWidth = ColumnWidth;
			DetailsViewArgs.bShowCustomFilterOption = PropertiesToShow.Num() != 0 || CategoriesToShow.Num() != 0;
			
			DetailViewWidget = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
			
			DetailViewWidget->SetCustomFilterLabel(LOCTEXT("ShowAllParameters", "Show All Parameters"));
			DetailViewWidget->SetCustomFilterDelegate(FSimpleDelegate::CreateUObject(this, &UDetailsView::ToggleShowingOnlyAllowedProperties));
			DetailViewWidget->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateUObject(this, &UDetailsView::GetIsPropertyVisible));
			DetailViewWidget->SetIsCustomRowVisibleDelegate(FIsCustomRowVisible::CreateUObject(this, &UDetailsView::GetIsRowVisible));
			DetailViewWidget->SetObject(ViewedObject);
			if (DetailViewWidget.IsValid())
			{
				GetDisplayWidget()->SetContent(DetailViewWidget.ToSharedRef());
				bCreateMissingWidget = false;
			}
			else
			{
				MissingWidgetText = FPropertyViewHelper::UnknownErrorText;
			}
		}
	}

	if (bCreateMissingWidget)
	{
		GetDisplayWidget()->SetContent(
			SNew(STextBlock)
			.Text(MissingWidgetText)
		);
	}
}


void UDetailsView::OnObjectChanged()
{
	UObject* ViewedObject = GetObject();
	if (DetailViewWidget.IsValid() && ViewedObject != nullptr)
	{
		DetailViewWidget->SetObject(ViewedObject);
	}
	else
	{
		BuildContentWidget();
	}
}


void UDetailsView::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	FNotifyHook::NotifyPostChange(PropertyChangedEvent, PropertyThatChanged);

	FName PropertyName = PropertyThatChanged ? PropertyThatChanged->GetFName() : NAME_None;
	OnPropertyChangedBroadcast(PropertyName);
}


void UDetailsView::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (IsDesignTime())
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, ViewIdentifier)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, bAllowFiltering)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, bAllowFavoriteSystem)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, bShowModifiedPropertiesOption)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, bShowKeyablePropertiesOption)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, bShowAnimatedPropertiesOption)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, bShowScrollBar)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, bForceHiddenPropertyVisibility)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDetailsView, ColumnWidth))
		{
			BuildContentWidget();
		}
	}
}


void UDetailsView::ToggleShowingOnlyAllowedProperties()
{
	bShowOnlyAllowed = !bShowOnlyAllowed;
	if (DetailViewWidget.IsValid())
	{
		DetailViewWidget->ForceRefresh();
	}
}

bool UDetailsView::IsRowVisibilityFiltered() const
{
	return bShowOnlyAllowed && (PropertiesToShow.Num() > 0 || CategoriesToShow.Num() > 0);
}

bool UDetailsView::GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	if (!IsRowVisibilityFiltered())
	{
		return true;
	}
	if (PropertiesToShow.Contains(PropertyAndParent.Property.GetFName()))
	{
		return true;
	}

	const FProperty* Property = PropertyAndParent.ParentProperties.Num() > 0 ? PropertyAndParent.ParentProperties.Last() : &PropertyAndParent.Property;

	// Check against the parent property name
	if (PropertiesToShow.Contains(Property->GetFName()))
	{
		return true;
	}

	// get the topmost parent's category name if the property has one
	const FString CategoryString = FObjectEditorUtils::GetCategoryFName(Property).ToString();
	FString SubcategoryString = CategoryString;
	
	int32 SubcategoryStart;
	if (CategoryString.FindChar(TEXT('|'), SubcategoryStart))
	{
		SubcategoryString = CategoryString.Left(SubcategoryStart);
	}

	if (CategoriesToShow.ContainsByPredicate([&CategoryString, &SubcategoryString](const FName& Element)
		{
			FString ElementString = Element.ToString();
			return CategoryString == ElementString || SubcategoryString == ElementString;
		}))
	{
		return true;
	}
	return false;
}

bool UDetailsView::GetIsRowVisible(FName InRowName, FName InParentName) const
{
	if (!IsRowVisibilityFiltered())
	{
		return true;
	}
	if (PropertiesToShow.Contains(InRowName))
	{
		return true;
	}
	if (CategoriesToShow.Contains(InParentName))
	{
		return true;
	}
	return false;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
