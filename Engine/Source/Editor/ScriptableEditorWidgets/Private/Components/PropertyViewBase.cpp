// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PropertyViewBase.h"

#include "Async/Async.h"
#include "Editor.h"
#include "Engine/World.h"
#include "PropertyEditorModule.h"
#include "Widgets/Layout/SBorder.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyViewBase)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UPropertyViewBase


void UPropertyViewBase::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	DisplayedWidget.Reset();
}


TSharedRef<SWidget> UPropertyViewBase::RebuildWidget()
{
	DisplayedWidget = SNew(SBorder)
		.Padding(0.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.BorderImage(FAppStyle::GetBrush("NoBorder"));
	
	BuildContentWidget();

	return DisplayedWidget.ToSharedRef();
}


UObject* UPropertyViewBase::GetObject() const
{
	return Object.Get();
}


void UPropertyViewBase::SetObject(UObject* InObject)
{
	if (Object.Get() != InObject)
	{
		Object = InObject;
		OnObjectChanged();
	}
}


void UPropertyViewBase::OnPropertyChangedBroadcast(FName PropertyName)
{
	OnPropertyChanged.Broadcast(PropertyName);
}


void UPropertyViewBase::PostLoad()
{
	Super::PostLoad();

	if (Object.IsNull() && !SoftObjectPath_DEPRECATED.IsNull())
	{
		Object = SoftObjectPath_DEPRECATED;
	}

	if(!Object.IsValid() && Object.ToSoftObjectPath().IsAsset() && bAutoLoadAsset && !HasAnyFlags(RF_BeginDestroyed))
	{
		Object.LoadSynchronous();
		BuildContentWidget();
	}
}


void UPropertyViewBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPropertyViewBase, Object))
	{
		OnObjectChanged();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


const FText UPropertyViewBase::GetPaletteCategory()
{
	return LOCTEXT("Editor", "Editor");
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
