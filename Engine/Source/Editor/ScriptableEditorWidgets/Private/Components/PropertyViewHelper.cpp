// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PropertyViewHelper.h"

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"


#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// FPropertyViewHelper

const FText FPropertyViewHelper::UndefinedObjectText = LOCTEXT("UndefinedObject", "Undefined Object");
const FText FPropertyViewHelper::UnloadedObjectText = LOCTEXT("UnloadedObject", "Unloaded Object");
const FText FPropertyViewHelper::UnknownErrorText = LOCTEXT("UnknownError", "Unknown Error");
const FText FPropertyViewHelper::EditorOnlyText = LOCTEXT("EditorWidget", "Editor Only");
const FText FPropertyViewHelper::UndefinedPropertyText = LOCTEXT("UndefinedProperty", "Undefined Property");
const FText FPropertyViewHelper::UnknownPropertyText = LOCTEXT("UnknownProperty", "Unknown Property");
const FText FPropertyViewHelper::InvalidPropertyText = LOCTEXT("InvalidPropertyName", "Invalid Property");
const FText FPropertyViewHelper::UnsupportedPropertyText = LOCTEXT("SinglePropertyView_UnsupporteddProperty", "Unsupported Property");

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE