// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FText;

struct FPropertyViewHelper
{
	static const FText UndefinedObjectText;
	static const FText UnloadedObjectText;
	static const FText UnknownErrorText;
	static const FText EditorOnlyText;
	static const FText UndefinedPropertyText;
	static const FText UnknownPropertyText;
	static const FText InvalidPropertyText;
	static const FText UnsupportedPropertyText;
};
