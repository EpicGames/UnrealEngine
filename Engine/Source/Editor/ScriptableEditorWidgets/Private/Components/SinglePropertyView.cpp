// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SinglePropertyView.h"

#include "Components/PropertyViewHelper.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "ISinglePropertyView.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SinglePropertyView)


#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// USinglePropertyView


void USinglePropertyView::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	SinglePropertyViewWidget.Reset();
}


void USinglePropertyView::BuildContentWidget()
{
	SinglePropertyViewWidget.Reset();

	if (!GetDisplayWidget().IsValid())
	{
		return;
	}

	bool bCreateMissingWidget = true;
	FText MissingWidgetText = FPropertyViewHelper::EditorOnlyText;

	if (GIsEditor)
	{
		UObject* ViewedObject = GetObject();
		if (ViewedObject == nullptr)
		{
			bool bIsObjectNull = Object.IsNull();
			if (bIsObjectNull)
			{
				MissingWidgetText = FPropertyViewHelper::UndefinedObjectText;
			}
			else
			{
				MissingWidgetText = FPropertyViewHelper::UnloadedObjectText;
			}
		}
		else if (PropertyName == NAME_None)
		{
			MissingWidgetText = FPropertyViewHelper::UndefinedPropertyText;
		}
		else
		{
			FProperty* Property = ViewedObject->GetClass()->FindPropertyByName(PropertyName);
			if (Property == nullptr)
			{
				MissingWidgetText = FPropertyViewHelper::UnknownPropertyText;
			}
			else if (!Property->HasAllPropertyFlags(CPF_Edit))
			{
				MissingWidgetText = FPropertyViewHelper::InvalidPropertyText;
			}
			else if (CastField<FArrayProperty>(Property) || CastField<FMapProperty>(Property) || CastField<FSetProperty>(Property))
			{
				MissingWidgetText = FPropertyViewHelper::UnsupportedPropertyText;
			}
			else
			{
				FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
				FSinglePropertyParams SinglePropertyArgs;
				SinglePropertyArgs.NameOverride = NameOverride;
				SinglePropertyViewWidget = PropertyEditorModule.CreateSingleProperty(ViewedObject, PropertyName, SinglePropertyArgs);

				if (SinglePropertyViewWidget.IsValid())
				{
					if (const TSharedPtr<IPropertyHandle> Handle = SinglePropertyViewWidget->GetPropertyHandle())
					{
						FSimpleDelegate PropertyChanged = FSimpleDelegate::CreateUObject(this, &USinglePropertyView::InternalSinglePropertyChanged);
						Handle->SetOnPropertyValueChanged(PropertyChanged);
						Handle->SetOnChildPropertyValueChanged(PropertyChanged);
					}
					GetDisplayWidget()->SetContent(SinglePropertyViewWidget.ToSharedRef());
					bCreateMissingWidget = false;
				}
				else
				{
					// Some built-in structs like FColor are supported, others aren't
					if (CastField<FStructProperty>(Property))
					{
						MissingWidgetText = FPropertyViewHelper::UnsupportedPropertyText;
					}
					else
					{
						MissingWidgetText = FPropertyViewHelper::UnknownErrorText;
					}
				}
			}
		}
	}

	if (bCreateMissingWidget)
	{
		GetDisplayWidget()->SetContent(
			SNew(STextBlock)
			.Text(MissingWidgetText)
		);
	}
}


FName USinglePropertyView::GetPropertyName() const
{
	return PropertyName;
}


void USinglePropertyView::SetPropertyName(FName InPropertyName)
{
	if (PropertyName != InPropertyName)
	{
		PropertyName = InPropertyName;
		BuildContentWidget();
	}
}

FText USinglePropertyView::GetNameOverride() const
{
	return NameOverride;
}


void USinglePropertyView::SetNameOverride(FText InNameOverride)
{
	if (!NameOverride.EqualTo(InNameOverride))
	{
		NameOverride = InNameOverride;
		BuildContentWidget();
	}
}

void USinglePropertyView::OnObjectChanged()
{
	BuildContentWidget();
}


void USinglePropertyView::InternalSinglePropertyChanged()
{
	OnPropertyChangedBroadcast(GetPropertyName());
}


void USinglePropertyView::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (IsDesignTime())
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USinglePropertyView, PropertyName)
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USinglePropertyView, NameOverride))
		{
			BuildContentWidget();
		}
	}
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
