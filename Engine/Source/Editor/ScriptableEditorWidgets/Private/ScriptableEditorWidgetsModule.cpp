// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableEditorWidgetsModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FScriptableEditorWidgetsModule, ScriptableEditorWidgets);


void FScriptableEditorWidgetsModule::StartupModule()
{

}

void FScriptableEditorWidgetsModule::ShutdownModule()
{

}



