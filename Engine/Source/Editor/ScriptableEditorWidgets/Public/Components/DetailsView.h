// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PropertyViewBase.h"
#include "Containers/Array.h"
#include "Misc/NotifyHook.h"
#include "PropertyEditorDelegates.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "DetailsView.generated.h"

#define UE_API SCRIPTABLEEDITORWIDGETS_API

class FProperty;
class IDetailsView;
class UObject;
struct FPropertyAndParent;
struct FPropertyChangedEvent;

/**
 * The details view allows you to display the value of an object properties.
 */
UCLASS(MinimalAPI)
class UDetailsView : public UPropertyViewBase, public FNotifyHook
{
	GENERATED_BODY()

public:
	/** True if we allow filtering through search and the filter dropdown menu. */
	UPROPERTY(EditAnywhere, Category = "View")
	bool bAllowFiltering = false;

	/** If false, the current properties editor will never display the favorite system */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View")
	bool bAllowFavoriteSystem = false;

	/** True if you want to show the 'Show Only Modified Properties'. Only valid in conjunction with bAllowFiltering */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View", meta = (EditCondition = "bAllowFiltering"))
	bool bShowModifiedPropertiesOption = false;

	/** True if you want to show the 'Show Only Keyable Properties'. Only valid in conjunction with bAllowFiltering */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View", meta = (EditCondition = "bAllowFiltering"))
	bool bShowKeyablePropertiesOption = true;

	/** True if you want to show the 'Show Only Animated Properties'. Only valid in conjunction with bAllowFiltering */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View", meta = (EditCondition = "bAllowFiltering"))
	bool bShowAnimatedPropertiesOption = true;

	/** The default column width */
	UPROPERTY(EditAnywhere, Category = "View", meta=(ClampMin="0.0", ClampMax="1.0"))
	float ColumnWidth = 0.65f;

	/** If false, the details panel's scrollbar will always be hidden. Useful when embedding details panels in widgets that either grow to accommodate them, or with scrollbars of their own. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View")
	bool bShowScrollBar = true;

	/** If true, all properties will be visible, not just those with CPF_Edit */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View")
	bool bForceHiddenPropertyVisibility = false;

	/** Identifier for this details view; NAME_None if this view is anonymous */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View")
	FName ViewIdentifier = NAME_None;

	/** Which categories to show in the details panel. If both this and the Properties To Show lists are empty, all properties will show. */
	UPROPERTY(EditAnywhere, Category = "View")
	TArray<FName> CategoriesToShow;

	/** Which properties to show in the details panel. If both this and the Categories To Show lists are empty, all properties will show. */
	UPROPERTY(EditAnywhere, Category = "View")
	TArray<FName> PropertiesToShow;

	//~ UPropertyViewBase interface
protected:
	UE_API virtual void BuildContentWidget() override;
	UE_API virtual void OnObjectChanged() override;
	//~ End of UPropertyViewBase interface

public:
	//~ UWidget interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UWidget interface

	//~ FNotifyHook interface
	UE_API virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	//~ End of FNotifyHook interface

	//~ UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End of UObject interface

protected:
	UE_API void ToggleShowingOnlyAllowedProperties();
	UE_API virtual bool IsRowVisibilityFiltered() const;
	UE_API virtual bool GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const;
	UE_API virtual bool GetIsRowVisible(FName InRowName, FName InParentName) const;

private:
	TSharedPtr<IDetailsView> DetailViewWidget;

	/** Showing properties in this details panel works by allowing only specific categories and properties. This flag enables you to show all properties without needing to specify. */
	UPROPERTY()
	bool bShowOnlyAllowed = true;
};

#undef UE_API
