// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/Widget.h"
#include "Widgets/SWidget.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "PropertyViewBase.generated.h"

#define UE_API SCRIPTABLEEDITORWIDGETS_API

class SBorder;

/** Sets a delegate called when the property value changes */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPropertyValueChanged, FName, PropertyName);


/**
 * Base of property view allows you to display the value of an object properties.
 */
UCLASS(MinimalAPI, Abstract)
class UPropertyViewBase : public UWidget
{
	GENERATED_BODY()

protected:
	/** The object to view. */
	UPROPERTY(meta = (DisplayName="Object"))
	TSoftObjectPtr<UObject> Object;

	UPROPERTY()
	FSoftObjectPath SoftObjectPath_DEPRECATED;

	/** Load the object (if it's an asset) when the widget is created. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View")
	bool bAutoLoadAsset = true;

	/** Sets a delegate called when the property value changes */
	UPROPERTY(BlueprintAssignable, Category = "View|Event")
	FOnPropertyValueChanged OnPropertyChanged;


public:
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API UObject* GetObject() const;

	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void SetObject(UObject* NewObject);

protected:
	virtual void BuildContentWidget() PURE_VIRTUAL(UPropertyViewBase::BuildContentWidget, );
	virtual void OnObjectChanged() { }
	TSharedPtr<SBorder> GetDisplayWidget() const { return DisplayedWidget; }
	UE_API void OnPropertyChangedBroadcast(FName PropertyName);

	//~ UWidget interface
public:
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual const FText GetPaletteCategory() override;

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End of UWidget interface

	//~ UObject interface
public:
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End of UObject interface

private:
	TSharedPtr<SBorder> DisplayedWidget;
};

#undef UE_API
