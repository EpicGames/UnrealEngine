// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PropertyViewBase.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#include "SinglePropertyView.generated.h"

#define UE_API SCRIPTABLEEDITORWIDGETS_API

class ISinglePropertyView;
class UObject;
struct FFrame;
struct FPropertyChangedEvent;

/**
 * The single property view allows you to display the value of an object's property.
 */
UCLASS(MinimalAPI)
class USinglePropertyView : public UPropertyViewBase
{
	GENERATED_BODY()

private:
	/** The name of the property to display. */
	UPROPERTY(EditAnywhere, Category = "View")
	FName PropertyName;

	/** Override for the property name that will be displayed instead of the property name. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "View")
	FText NameOverride;

public:
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API FName GetPropertyName() const;

	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void SetPropertyName(FName NewPropertyName);

	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API FText GetNameOverride() const;

	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void SetNameOverride(FText NewPropertyName);

private:
	UE_API void InternalSinglePropertyChanged();

	//~ UPropertyViewBase interface
protected:
	UE_API virtual void BuildContentWidget() override;
	UE_API virtual void OnObjectChanged() override;
	//~ End of UPropertyViewBase interface

	//~ UWidget interface
public:
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End of UWidget interface

	// UObject interface
public:
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

private:
	TSharedPtr<ISinglePropertyView> SinglePropertyViewWidget;
};

#undef UE_API
