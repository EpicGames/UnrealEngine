// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ScriptableEditorWidgets : ModuleRules
{
	public ScriptableEditorWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string [] {
				"Core",
				"Engine",
				"Slate",
				"SlateCore",
			}
		);
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"UMG",		
			}
		);
	}
}
